{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Blendspace Motion Analysis",
	"Description": "Allows analysis of locomotion/root motion properties in blend spaces",
	"Category": "BlendSpace",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": 
	[
		{
			"Name": "BlendSpaceMotionAnalysis",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
