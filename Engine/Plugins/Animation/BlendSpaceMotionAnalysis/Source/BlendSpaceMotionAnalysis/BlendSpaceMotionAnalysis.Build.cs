// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class BlendSpaceMotionAnalysis : ModuleRules
	{
		public BlendSpaceMotionAnalysis(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Persona",
				}
			);
		}
	}
}
