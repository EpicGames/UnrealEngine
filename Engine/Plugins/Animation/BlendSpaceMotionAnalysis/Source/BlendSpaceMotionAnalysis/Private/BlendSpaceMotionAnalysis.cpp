// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendSpaceMotionAnalysis.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendSpaceMotionAnalysis)

IMPLEMENT_MODULE(FBlendSpaceMotionAnalysis, BlendSpaceMotionAnalysis)

DEFINE_LOG_CATEGORY_STATIC(LogBlendSpaceMotionAnalysis, Log, All);

#define LOCTEXT_NAMESPACE "BlendSpaceMotionAnalysis"

//======================================================================================================================
class FBlendSpaceMotionAnalysisFeature : public IBlendSpaceAnalysisFeature 
{
public:
	// This should process the animation according to the analysis properties, or return false if that is not possible.
	bool CalculateSampleValue(float&                     Result,
							  const UBlendSpace&         BlendSpace,
							  const UAnalysisProperties* AnalysisProperties,
							  const UAnimSequence&       Animation,
							  const float                RateScale) const override;

	// This should return an instance derived from UAnalysisProperties that is suitable for the Function. The caller
	// will pass in a suitable owning object, outer, that the implementation should assign as owner of the newly created
	// object. 
	UAnalysisProperties* MakeAnalysisProperties(UObject* Outer, const FString& FunctionName) const override;

	// This should return the names of the functions handled
	TArray<FString> GetAnalysisFunctions() const override;
};

static FBlendSpaceMotionAnalysisFeature BlendSpaceMotionAnalysisFeature;

//======================================================================================================================
TArray<FString> FBlendSpaceMotionAnalysisFeature::GetAnalysisFunctions() const
{
	TArray<FString> Functions = 
	{
		TEXT("Locomotion"),
		TEXT("RootMotion")
	};
	return Functions;
}

//======================================================================================================================
UAnalysisProperties* FBlendSpaceMotionAnalysisFeature::MakeAnalysisProperties(UObject* Outer, const FString& FunctionName) const
{
	UAnalysisProperties* Result = nullptr;
	if (FunctionName.Equals(TEXT("RootMotion")))
	{
		Result = NewObject<URootMotionAnalysisProperties>(Outer);
	}
	else if (FunctionName.Equals(TEXT("Locomotion")))
	{
		Result = NewObject<ULocomotionAnalysisProperties>(Outer);
	}

	if (Result)
	{
		Result->Function = FunctionName;
	}
	return Result;
}

//======================================================================================================================
bool FBlendSpaceMotionAnalysisFeature::CalculateSampleValue(float&                     Result,
															const UBlendSpace&         BlendSpace,
															const UAnalysisProperties* AnalysisProperties,
															const UAnimSequence&       Animation,
															const float                RateScale) const
{
	if (!AnalysisProperties)
	{
		return false;
	}
	const FString& FunctionName = AnalysisProperties->Function;
	if (FunctionName.Equals(TEXT("RootMotion")))
	{
		return CalculateRootMotion(
			Result, BlendSpace, Cast<URootMotionAnalysisProperties>(AnalysisProperties), Animation, RateScale);
	}
	else if (FunctionName.Equals(TEXT("Locomotion")))
	{
		return CalculateLocomotion(
			Result, BlendSpace, Cast<ULocomotionAnalysisProperties>(AnalysisProperties), Animation, RateScale);
	}
	return false;
}

//======================================================================================================================
void FBlendSpaceMotionAnalysis::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(IBlendSpaceAnalysisFeature::GetModuleFeatureName(), &BlendSpaceMotionAnalysisFeature);
}

void FBlendSpaceMotionAnalysis::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IBlendSpaceAnalysisFeature::GetModuleFeatureName(), &BlendSpaceMotionAnalysisFeature);
}

#undef LOCTEXT_NAMESPACE
