// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocomotionAnalysis.h"
#include "BlendSpaceMotionAnalysis.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocomotionAnalysis)

DEFINE_LOG_CATEGORY_STATIC(LogLocomotionAnalysis, Log, All);

#define LOCTEXT_NAMESPACE "LocomotionAnalysis"

//======================================================================================================================
void ULocomotionAnalysisProperties::InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache)
{
	if (Cache)
	{
		UCachedAnalysisProperties* CachePtr = Cache.Get();
		UCachedMotionAnalysisProperties* CachedMotionAnalysisPropertiesPtr = Cast<UCachedMotionAnalysisProperties>(CachePtr);
		if (!CachedMotionAnalysisPropertiesPtr)
		{
			// Replace any existing cache if it is not our own type
			CachedMotionAnalysisPropertiesPtr = NewObject<UCachedMotionAnalysisProperties>(CachePtr->GetOuter());
			CachedMotionAnalysisPropertiesPtr->CopyFrom(*CachePtr);
			Cache = CachedMotionAnalysisPropertiesPtr;
		}
		FunctionAxis = CachedMotionAnalysisPropertiesPtr->LocomotionFunctionAxis;
		PrimaryBoneSocket = CachedMotionAnalysisPropertiesPtr->BoneSocket1;
		SecondaryBoneSocket = CachedMotionAnalysisPropertiesPtr->BoneSocket2;
		CharacterFacingAxis = CachedMotionAnalysisPropertiesPtr->CharacterFacingAxis;
		CharacterUpAxis = CachedMotionAnalysisPropertiesPtr->CharacterUpAxis;
	}
}

//======================================================================================================================
void ULocomotionAnalysisProperties::MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace)
{
	UCachedMotionAnalysisProperties* CachedMotionAnalysisPropertiesPtr = Cast<UCachedMotionAnalysisProperties>(Cache.Get());
	if (!CachedMotionAnalysisPropertiesPtr)
	{
		// Replace any existing cache if it is not our own type
		CachedMotionAnalysisPropertiesPtr = NewObject<UCachedMotionAnalysisProperties>(BlendSpace);
		if (Cache.Get())
		{
			CachedMotionAnalysisPropertiesPtr->CopyFrom(*Cache.Get());
		}
		Cache = CachedMotionAnalysisPropertiesPtr;
	}
	CachedMotionAnalysisPropertiesPtr->LocomotionFunctionAxis = FunctionAxis;
	CachedMotionAnalysisPropertiesPtr->BoneSocket1 = PrimaryBoneSocket;
	CachedMotionAnalysisPropertiesPtr->BoneSocket2 = SecondaryBoneSocket;
	CachedMotionAnalysisPropertiesPtr->CharacterFacingAxis = CharacterFacingAxis;
	CachedMotionAnalysisPropertiesPtr->CharacterUpAxis = CharacterUpAxis;
}

//======================================================================================================================
// Note that if a looping animation has 56 keys, then its first key is 0 and last is 55, but these will be identical poses.
// Thus it has one fewer intervals/unique keys
static bool CalculateLocomotionVelocity(
	FVector&                             Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const FBoneSocketTarget&             BoneSocket,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	const int32 NumSampledKeys = Animation.GetNumberOfSampledKeys();
	if (!AnalysisProperties || NumSampledKeys <= 0)
	{
		return false;
	}
	if (NumSampledKeys == 1)
	{
		Result.Set(0, 0, 0);
		return true;
	}

	FTransform BoneOffset;
	FName BoneName;
	if (!BlendSpaceAnalysis::GetBoneInfo(Animation, BoneSocket, BoneOffset, BoneName))
	{
		return false;
	}

	// Note that for locomotion we don't support the frame changing
	FTransform FrameTM = FTransform::Identity;
	FVector FrameFacingDir, FrameUpDir, FrameRightDir;
	BlendSpaceAnalysis::GetFrameDirs(FrameFacingDir, FrameUpDir, FrameRightDir, FrameTM, AnalysisProperties);

	// The frame time delta
	double DeltaTime = Animation.GetPlayLength() / double(NumSampledKeys);

	// First step is to figure out the approximate direction. Note that the average velocity will be zero (assuming a
	// complete cycle) - but if we apply a weight that is based on the height, then we can bias it towards the foot that
	// is on the ground.
	double MinHeight = DBL_MAX;
	double MaxHeight = -DBL_MAX;
	FVector AveragePos(0);
	TArray<FVector> Positions;
	Positions.SetNum(NumSampledKeys);
	for (int32 Key = 0; Key != NumSampledKeys; ++Key)
	{
		FTransform BoneTM = BlendSpaceAnalysis::GetBoneTransform(Animation, Key, BoneName);
		FTransform TM = BoneOffset * BoneTM;
		FVector Pos = TM.GetTranslation();
		Positions[Key] = Pos;
		double Height = Pos | FrameUpDir;
		MinHeight = FMath::Min(MinHeight, Height);
		MaxHeight = FMath::Max(MaxHeight, Height);
		AveragePos += Pos;
#ifdef ANALYSIS_VERBOSE_LOG
		UE_LOGF(LogAnimation, Log, "Pos %f %f %f", Pos.X, Pos.Y, Pos.Z);
#endif
	}
	AveragePos /= NumSampledKeys;

	// Calculate velocities. 
	TArray<FVector> Velocities;
	Velocities.SetNum(NumSampledKeys);
	for (int32 Key = 0; Key != NumSampledKeys; ++Key)
	{
		int32 PrevKey = (Key + NumSampledKeys - 1) % NumSampledKeys;
		int32 NextKey = (Key +  1) % NumSampledKeys;
		Velocities[Key] = (Positions[NextKey] - Positions[PrevKey]) / (2.0 * DeltaTime);
	}

	FVector BiasedFootVel(0);
	double TotalWeight = 0;
	for (int32 Key = 0 ; Key != NumSampledKeys ; ++Key)
	{
		double Height = Positions[Key] | FrameUpDir;
		double Weight = 1.0 - (Height - MinHeight) / (MaxHeight - MinHeight);
		BiasedFootVel += Velocities[Key] * Weight;
		TotalWeight += Weight;
	}
	BiasedFootVel /= TotalWeight; 

	if (BiasedFootVel.IsNearlyZero())
	{
		Result.Set(0, 0, 0);
		return true;
	}

	FVector ApproxLocoDir = -BiasedFootVel.GetSafeNormal();

	// Now we can form a mask, where 0 means traveling in the wrong direction (so clearly off the ground), and positive
	// numbers will indicate how far into a valid segment we are. We will assume that the animation is looping
	TArray<int32> Mask;
	Mask.SetNum(NumSampledKeys);
	for (int32 Key = 0 ; Key != NumSampledKeys ; ++Key)
	{
		Mask[Key] = (Velocities[Key] | ApproxLocoDir) >= 0 ? 0 : 1;
	}

	int32 StartKey = -1;
	TArray<int32> Mask1 = Mask;
	bool bRepeat = true;
	int32 MaxMask = 0;
	int32 PrevNumFound = NumSampledKeys + 1;
	while (bRepeat)
	{
		bRepeat = false;
		int32 NumFound = 0;
		for (int32 Key = 0 ; Key != NumSampledKeys ; ++Key)
		{
			if (Mask[Key] > 0)
			{
				int32 PrevKey = (Key + NumSampledKeys - 1) % NumSampledKeys;
				int32 NextKey = (Key + 1) % NumSampledKeys;
				if (Mask[PrevKey] == Mask[Key] && Mask[NextKey] == Mask[Key])
				{
					++Mask1[Key];
					MaxMask = FMath::Max(MaxMask, Mask1[Key]);
					bRepeat = true;
					++NumFound;
				}
			}
		}
		Mask = Mask1;
		// Avoid a perpetual loop (e.g. can happen if initially all the mask values are 1... though that shouldn't
		// really happen).
		if (NumFound >= PrevNumFound)
		{
			bRepeat = false;
		}
		else
		{
			PrevNumFound = NumFound;
		}
	}

	// When searching we will want to start outside of a "good" region.
	int32 AZeroKey = 0;
	for (int32 Key = 0 ; Key != NumSampledKeys ; ++Key)
	{
		if (Mask[Key] == 0)
		{
			AZeroKey = Key;
		}
	}

	// We use the mask (with a somewhat arbitrary threshold) to get rid of velocities that are near to the foot
	// plant/take-off time (and might be when the foot is in the time). Then We look for the highest velocity. Note that
	// if we're being called with the foot (ankle) joint, it will tend to underestimate velocities since it is nearer
	// the hip than the ground contact point.
	// 
	int32 Threshold = FMath::Max(MaxMask / 2, 1);
	int32 Num = 0;
	FVector AverageFootVel(0);
	double BestSpeed = 0;
	int32 BestSpeedKey = 0;
	for (int32 K = AZeroKey ; K != AZeroKey + NumSampledKeys ; ++K)
	{
		int32 Key = K % NumSampledKeys;
		if (Mask[Key] >= Threshold)
		{
			double Speed = Velocities[Key].Size();
#ifdef ANALYSIS_VERBOSE_LOG
			UE_LOGF(LogAnimation, Log, "Candidate %d Mask %d vel = %f %f %f, speed = %f", 
				   Key, Mask[Key], Velocities[Key].X, Velocities[Key].Y, Velocities[Key].Z, Speed);
#endif
			if (Speed > BestSpeed)
			{
				BestSpeed = Speed;
				BestSpeedKey = Key;
			}
		}
		else
		{
			if (BestSpeed > 0)
			{
#ifdef ANALYSIS_VERBOSE_LOG
				UE_LOGF(LogAnimation, Log, "Picked Candidate %d vel = %f %f %f, speed = %f", 
					   BestSpeedKey, Velocities[BestSpeedKey].X, Velocities[BestSpeedKey].Y, Velocities[BestSpeedKey].Z, BestSpeed);
#endif
				AverageFootVel += Velocities[BestSpeedKey];
				++Num;
				BestSpeed = 0;
			}
		}
	}
	// Make sure we didn't miss the last data point
	if (BestSpeed > 0)
	{
#ifdef ANALYSIS_VERBOSE_LOG
		UE_LOGF(LogAnimation, Log, "Picked Candidate %d vel = %f %f %f, speed = %f", 
			   BestSpeedKey, Velocities[BestSpeedKey].X, Velocities[BestSpeedKey].Y, Velocities[BestSpeedKey].Z, BestSpeed);
#endif
		AverageFootVel += Velocities[BestSpeedKey];
		++Num;
		BestSpeed = 0;
	}

	AverageFootVel /= Num;
	double FacingVel = -AverageFootVel | FrameFacingDir;
	double RightVel = -AverageFootVel | FrameRightDir;
	double UpVel = -AverageFootVel | FrameUpDir;

	Result.Set(FacingVel, RightVel, UpVel);
	Result *= Animation.RateScale * RateScale;
	UE_LOGF(LogAnimation, Log, "%ls Locomotion vel = %f %f %f", *BoneName.ToString(), Result.X, Result.Y, Result.Z);
	return true;
}

//======================================================================================================================
static bool CalculateLocomotionVelocity(
	FVector&                             Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	if (!AnalysisProperties)
	{
		return false;
	}

	FVector Result1, Result2;
	int32 Num = 0;
	Result.Set(0, 0, 0);
	if (CalculateLocomotionVelocity(Result1, BlendSpace, AnalysisProperties, 
									AnalysisProperties->PrimaryBoneSocket, Animation, RateScale))
	{
#ifdef ANALYSIS_VERBOSE_LOG
		UE_LOGF(LogAnimation, Log, "Loco vel from primary = %f %f %f", Result1.X, Result1.Y, Result1.Z);
#endif
		Result += Result1;
		++Num;
	}
	if (CalculateLocomotionVelocity(Result2, BlendSpace, AnalysisProperties, 
									AnalysisProperties->SecondaryBoneSocket, Animation, RateScale))
	{
#ifdef ANALYSIS_VERBOSE_LOG
		UE_LOGF(LogAnimation, Log, "Loco vel from secondary = %f %f %f", Result2.X, Result2.Y, Result2.Z);
#endif
		Result += Result2;
		++Num;
	}
	if (Num)
	{
		Result /= Num;
		UE_LOGF(LogAnimation, Log, "Loco vel = %f %f %f", Result.X, Result.Y, Result.Z);
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion speed (magnitude)
static bool CalculateLocomotionSpeed(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateLocomotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Movement.Size();
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion direction (degrees)
static bool CalculateLocomotionDirection(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateLocomotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = FMath::RadiansToDegrees(FMath::Atan2(Movement.Y, Movement.X));
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion speed in the character's facing direction
static bool CalculateLocomotionForwardSpeed(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateLocomotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Movement.X;
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion speed in the character's upwards direction
static bool CalculateLocomotionUpwardSpeed(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateLocomotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Movement.Z;
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion speed in the character's right direction
static bool CalculateLocomotionRightwardSpeed(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateLocomotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Movement.Y;
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion slope angle (degrees) going in the facing direction
static bool CalculateLocomotionForwardSlope(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateLocomotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		if (Movement.X >= 0)
		{
			Result = FMath::RadiansToDegrees(FMath::Atan2(Movement.Z, Movement.X));
		}
		else
		{
			Result = FMath::RadiansToDegrees(FMath::Atan2(-Movement.Z, -Movement.X));
		}

		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion slope angle (degrees) going in the rightwards direction
static bool CalculateLocomotionRightwardSlope(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateLocomotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		if (Movement.Y > 0)
		{
			Result = FMath::RadiansToDegrees(FMath::Atan2(Movement.Z, Movement.Y));
		}
		else
		{
			Result = FMath::RadiansToDegrees(FMath::Atan2(-Movement.Z, -Movement.Y));
		}
		return true;
	}
	return false;
}

//======================================================================================================================
static bool CalculateLocomotion(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	if (!AnalysisProperties)
	{
		return false;

	}
	switch (AnalysisProperties->FunctionAxis)
	{
	case EAnalysisLocomotionAxis::Speed:
		return CalculateLocomotionSpeed(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisLocomotionAxis::Direction:
		return CalculateLocomotionDirection(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisLocomotionAxis::ForwardSpeed:
		return CalculateLocomotionForwardSpeed(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisLocomotionAxis::RightwardSpeed:
		return CalculateLocomotionRightwardSpeed(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisLocomotionAxis::UpwardSpeed:
		return CalculateLocomotionUpwardSpeed(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisLocomotionAxis::ForwardSlope:
		return CalculateLocomotionForwardSlope(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisLocomotionAxis::RightwardSlope:
		return CalculateLocomotionRightwardSlope(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	default:
		return false;
	}
}

//======================================================================================================================
// Note that it is easier to do the calculations involving world-space positions in doubles, and
// then cast Result to float here, than it is to be casting in all the functions above.
bool CalculateLocomotion(
	float&                               Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	double ResultDouble = Result;
	bool bResult = CalculateLocomotion(ResultDouble, BlendSpace, AnalysisProperties, Animation, RateScale);
	Result = FloatCastChecked<float>(ResultDouble, UE::LWC::DefaultFloatPrecision);
	return bResult;
}

#undef LOCTEXT_NAMESPACE
