// Copyright Epic Games, Inc. All Rights Reserved.

#include "RootMotionAnalysis.h"
#include "BlendSpaceMotionAnalysis.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionAnalysis)

DEFINE_LOG_CATEGORY_STATIC(LogRootMotionAnalysis, Log, All);

#define LOCTEXT_NAMESPACE "RootMotionAnalysis"

//======================================================================================================================
void URootMotionAnalysisProperties::InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache)
{
	if (Cache)
	{
		UCachedAnalysisProperties* CachePtr = Cache.Get();
		UCachedMotionAnalysisProperties* CachedMotionAnalysisPropertiesPtr = Cast<UCachedMotionAnalysisProperties>(CachePtr);
		if (!CachedMotionAnalysisPropertiesPtr)
		{
			// Replace any existing cache if it is not our own type
			CachedMotionAnalysisPropertiesPtr = NewObject<UCachedMotionAnalysisProperties>(CachePtr->GetOuter());
			CachedMotionAnalysisPropertiesPtr->CopyFrom(*CachePtr);
			Cache = CachedMotionAnalysisPropertiesPtr;
		}
		FunctionAxis = CachedMotionAnalysisPropertiesPtr->RootMotionFunctionAxis;
		BoneSocket = CachedMotionAnalysisPropertiesPtr->BoneSocket1;
		Space = CachedMotionAnalysisPropertiesPtr->Space;
		SpaceBoneSocket = CachedMotionAnalysisPropertiesPtr->SpaceBoneSocket;
		CharacterFacingAxis = CachedMotionAnalysisPropertiesPtr->CharacterFacingAxis;
		CharacterUpAxis = CachedMotionAnalysisPropertiesPtr->CharacterUpAxis;
		StartTimeFraction = CachedMotionAnalysisPropertiesPtr->StartTimeFraction;
		EndTimeFraction = CachedMotionAnalysisPropertiesPtr->EndTimeFraction;
	}
}

//======================================================================================================================
void URootMotionAnalysisProperties::MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace)
{
	UCachedMotionAnalysisProperties* CachedMotionAnalysisPropertiesPtr = Cast<UCachedMotionAnalysisProperties>(Cache.Get());
	if (!CachedMotionAnalysisPropertiesPtr)
	{
		// Replace any existing cache if it is not our own type
		CachedMotionAnalysisPropertiesPtr = NewObject<UCachedMotionAnalysisProperties>(BlendSpace);
		if (Cache.Get())
		{
			CachedMotionAnalysisPropertiesPtr->CopyFrom(*Cache.Get());
		}
		Cache = CachedMotionAnalysisPropertiesPtr;
	}
	CachedMotionAnalysisPropertiesPtr->RootMotionFunctionAxis = FunctionAxis;
	CachedMotionAnalysisPropertiesPtr->BoneSocket1 = BoneSocket;
	CachedMotionAnalysisPropertiesPtr->Space = Space;
	CachedMotionAnalysisPropertiesPtr->SpaceBoneSocket = SpaceBoneSocket;
	CachedMotionAnalysisPropertiesPtr->CharacterFacingAxis = CharacterFacingAxis;
	CachedMotionAnalysisPropertiesPtr->CharacterUpAxis = CharacterUpAxis;
	CachedMotionAnalysisPropertiesPtr->StartTimeFraction = StartTimeFraction;
	CachedMotionAnalysisPropertiesPtr->EndTimeFraction = EndTimeFraction;
}

//======================================================================================================================
// Calculates the movement speed (magnitude) 
static bool CalculateRootMotionVelocity(
	FVector&                             Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Velocity;
	// Todo: we can remove the static_cast once the templated version of BlendSpaceAnalysis::CalculateVelocity is removed in 5.8+
	if (BlendSpaceAnalysis::CalculateVelocity(Velocity, BlendSpace, static_cast<const ULinearAnalysisPropertiesBase*>(AnalysisProperties), Animation, RateScale))
	{
		FTransform FrameTM;
		FVector FrameFacingDir = BlendSpaceAnalysis::GetAxisFromTM(FrameTM, AnalysisProperties->CharacterFacingAxis);
		FVector FrameUpDir = BlendSpaceAnalysis::GetAxisFromTM(FrameTM, AnalysisProperties->CharacterUpAxis);
		FVector FrameRightDir = FVector::CrossProduct(FrameUpDir, FrameFacingDir);
		double ForwardVel = Velocity | FrameFacingDir;
		double RightVel = Velocity | FrameRightDir;
		double UpVel = Velocity | FrameUpDir;
		Result.Set(ForwardVel, RightVel, UpVel);
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the movement speed (magnitude) 
static bool CalculateRootMotionSpeed(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateRootMotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Movement.Size();
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the movement speed (magnitude) 
static bool CalculateRootMotionDirection(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateRootMotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = FMath::RadiansToDegrees(FMath::Atan2(Movement.Y, Movement.X));
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion speed in the character's facing direction
static bool CalculateRootMotionForwardSpeed(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateRootMotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Movement.X;
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion speed in the character's upwards direction
static bool CalculateRootMotionUpwardSpeed(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateRootMotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Movement.Z;
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion speed in the character's right direction
static bool CalculateRootMotionRightwardSpeed(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateRootMotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Movement.Y;
		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion slope angle (degrees) going in the facing direction
static bool CalculateRootMotionForwardSlope(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateRootMotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		if (Movement.X >= 0)
		{
			Result = FMath::RadiansToDegrees(FMath::Atan2(Movement.Z, Movement.X));
		}
		else
		{
			Result = FMath::RadiansToDegrees(FMath::Atan2(-Movement.Z, -Movement.X));
		}

		return true;
	}
	return false;
}

//======================================================================================================================
// Calculates the locomotion slope angle (degrees) going in the rightwards direction
static bool CalculateRootMotionRightwardSlope(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	FVector Movement;
	if (CalculateRootMotionVelocity(Movement, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		if (Movement.Y > 0)
		{
			Result = FMath::RadiansToDegrees(FMath::Atan2(Movement.Z, Movement.Y));
		}
		else
		{
			Result = FMath::RadiansToDegrees(FMath::Atan2(-Movement.Z, -Movement.Y));
		}
		return true;
	}
	return false;
}


//======================================================================================================================
static bool CalculateRootMotion(
	double&                              Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	if (!AnalysisProperties)
	{
		return false;
	}

	switch (AnalysisProperties->FunctionAxis)
	{
	case EAnalysisRootMotionAxis::Speed:
		return CalculateRootMotionSpeed(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisRootMotionAxis::Direction:
		return CalculateRootMotionDirection(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisRootMotionAxis::ForwardSpeed:
		return CalculateRootMotionForwardSpeed(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisRootMotionAxis::RightwardSpeed:
		return CalculateRootMotionRightwardSpeed(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisRootMotionAxis::UpwardSpeed:
		return CalculateRootMotionUpwardSpeed(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisRootMotionAxis::ForwardSlope:
		return CalculateRootMotionForwardSlope(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	case EAnalysisRootMotionAxis::RightwardSlope:
		return CalculateRootMotionRightwardSlope(Result, BlendSpace, AnalysisProperties, Animation, RateScale);
	default:
		return false;
	}
}

//======================================================================================================================
// Note that it is easier to do the calculations involving world-space positions in doubles, and
// then cast Result to float here, than it is to be casting in all the functions above.
bool CalculateRootMotion(
	float&                               Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale)
{
	double DoubleResult = Result;
	bool bResult = CalculateRootMotion(DoubleResult, BlendSpace, AnalysisProperties, Animation, RateScale);
	Result = FloatCastChecked<float>(DoubleResult, UE::LWC::DefaultFloatPrecision);
	return bResult;
}

#undef LOCTEXT_NAMESPACE
