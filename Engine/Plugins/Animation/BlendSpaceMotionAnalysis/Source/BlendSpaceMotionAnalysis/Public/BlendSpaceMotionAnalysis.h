// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "RootMotionAnalysis.h"
#include "LocomotionAnalysis.h"
#include "BlendSpaceMotionAnalysis.generated.h"

#define UE_API BLENDSPACEMOTIONANALYSIS_API

//======================================================================================================================
UCLASS()
class UCachedMotionAnalysisProperties : public UCachedAnalysisProperties
{
	GENERATED_BODY()
public:
	EAnalysisRootMotionAxis RootMotionFunctionAxis = EAnalysisRootMotionAxis::Speed;
	EAnalysisLocomotionAxis LocomotionFunctionAxis = EAnalysisLocomotionAxis::Speed;
};

//======================================================================================================================
class FBlendSpaceMotionAnalysis : public IModuleInterface 
{
public:
	UE_API void StartupModule() override;
	UE_API void ShutdownModule() override;
};

#undef UE_API
