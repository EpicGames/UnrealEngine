// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlendSpaceAnalysis.h"
#include "LocomotionAnalysis.generated.h"

//======================================================================================================================
UENUM()
enum class EAnalysisLocomotionAxis : uint8
{
	Speed,
	Direction,
	ForwardSpeed,
	RightwardSpeed,
	UpwardSpeed,
	ForwardSlope,
	RightwardSlope,
};

//======================================================================================================================
UCLASS()
class ULocomotionAnalysisProperties : public UAnalysisProperties
{
	GENERATED_BODY()
public:
	void InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache) override;
	void MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace) override;

	/** Axis for the analysis function */
	UPROPERTY(EditAnywhere, DisplayName = "Axis", Category = AnalysisProperties)
	EAnalysisLocomotionAxis FunctionAxis = EAnalysisLocomotionAxis::Speed;

	/** The primary bone or socket used for analysis */
	UPROPERTY(EditAnywhere, DisplayName = "Bone/Socket 1", Category = AnalysisProperties)
	FBoneSocketTarget PrimaryBoneSocket;

	/** The secondary bone or socket used for analysis */
	UPROPERTY(EditAnywhere, DisplayName = "Bone/Socket 2", Category = AnalysisProperties)
	FBoneSocketTarget SecondaryBoneSocket;

	/** World or bone/socket axis that specifies the character's facing direction */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisLinearAxis CharacterFacingAxis = EAnalysisLinearAxis::PlusY;

	/** World or bone/socket axis that specifies the character's up direction */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisLinearAxis CharacterUpAxis = EAnalysisLinearAxis::PlusZ;
};

/**
 * Calculates the locomotion speed from the animation (which may be playrate scaled), 
 * according to the analysis properties.
 */
bool CalculateLocomotion(
	float&                               Result,
	const UBlendSpace&                   BlendSpace,
	const ULocomotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale);
