// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlendSpaceAnalysis.h"
#include "RootMotionAnalysis.generated.h"

//======================================================================================================================
UENUM()
enum class EAnalysisRootMotionAxis : uint8
{
	Speed,
	Direction,
	ForwardSpeed,
	RightwardSpeed,
	UpwardSpeed,
	ForwardSlope,
	RightwardSlope,
};

//======================================================================================================================
UCLASS()
class URootMotionAnalysisProperties : public ULinearAnalysisPropertiesBase
{
	GENERATED_BODY()
public:
	void InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache) override;
	void MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace) override;

	/** Axis for the analysis function */
	UPROPERTY(EditAnywhere, DisplayName = "Axis", Category = AnalysisProperties)
	EAnalysisRootMotionAxis FunctionAxis = EAnalysisRootMotionAxis::Speed;

	/** World or bone/socket axis that specifies the character's facing direction */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisLinearAxis CharacterFacingAxis = EAnalysisLinearAxis::PlusY;

	/** World or bone/socket axis that specifies the character's up direction */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisLinearAxis CharacterUpAxis = EAnalysisLinearAxis::PlusZ;
};

/**
 * Calculates the root motion movement speed from the animation (which may be playrate scaled), 
 * according to the analysis properties.
 */
bool CalculateRootMotion(
	float&                               Result,
	const UBlendSpace&                   BlendSpace,
	const URootMotionAnalysisProperties* AnalysisProperties,
	const UAnimSequence&                 Animation,
	const float                          RateScale);
